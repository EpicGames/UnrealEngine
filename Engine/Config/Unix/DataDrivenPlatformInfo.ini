[DataDrivenPlatformInfo]
bIsConfidential=false
IniParent=VulkanPC
bIsFakePlatform=true
GlobalIdentifier=969267E8F7A6414DAD0450C1CAE83A6E